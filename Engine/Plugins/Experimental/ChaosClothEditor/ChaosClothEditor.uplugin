{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Chaos Cloth Editor (Deprecated)",
	"Description": "Deprecated plugin, please use the Chaos Cloth plugin instead.",
	"Category": "Physics",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"Installed": false,
	"Modules": [],
	"Plugins": [
		{
			"Name": "ChaosCloth",
			"Enabled": true
		}
	]
}
